// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

struct FSlateBrush;

/**
 * The public interface to this module
 */
class IUATHelperModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IUATHelperModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IUATHelperModule >( "UATHelper" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "UATHelper" );
	}

	/** Used to callback into calling code when a UAT task completes. First param is the result type, second param is the runtime in sec. */
	typedef TFunction<void(FString, double)> UatTaskResultCallack;

	/** Creates and starts up a UAT Task
	  * @param	ResultLocation	The folder where the result of the task will be stored
	  * @param	ExtraSubText	Optional subtext displayed below the notification title while the task is in progress and on success
	  */
	virtual void CreateUatTask(const FString& CommandLine, const FText& PlatformDisplayName, const FText& TaskName, const FText &TaskShortName, const FSlateBrush* TaskIcon, const TArray<struct FAnalyticsEventAttribute>* OptionalAnalyticsParamArray = nullptr, UatTaskResultCallack ResultCallback = UatTaskResultCallack(), const FString& ResultLocation = FString(), const FText& ExtraSubText = FText::GetEmpty()) = 0;
};

