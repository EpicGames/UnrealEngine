// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class UATHelper : ModuleRules
	{
		public UATHelper(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"Slate",
					"SlateCore",
					"GameProjectGeneration",
					"EditorFramework",
					"UnrealEd",
					"Analytics",
					"OutputLog",
					"Json"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"MessageLog",
				}
			);
		}
	}
}
