// Copyright Epic Games, Inc. All Rights Reserved.

#include "UATHelperModule.h"
#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Stats/Stats.h"
#include "Misc/MonitoredProcess.h"
#include "Modules/ModuleManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "Framework/Docking/TabManager.h"
#include "Editor.h"
#include "EditorAnalytics.h"
#include "IUATHelperModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Serialization/JsonSerializer.h"

#include "Interfaces/IMainFrameModule.h"
#include "Editor/EditorPerProjectUserSettings.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "Misc/UObjectToken.h"

#include "GameProjectGenerationModule.h"
#include "AnalyticsEventAttribute.h"

#include "ShaderCompiler.h"
#include "OutputLogModule.h"

#define LOCTEXT_NAMESPACE "UATHelper"

DEFINE_LOG_CATEGORY_STATIC(UATHelper, Log, All);

/* Event Data
*****************************************************************************/

struct EventData
{
	FString EventName;
	bool bProjectHasCode;
	double StartTime;
	IUATHelperModule::UatTaskResultCallack ResultCallback;
};

/* FMainFrameActionCallbacks callbacks
*****************************************************************************/

class FMainFrameActionsNotificationTask
{
public:

	FMainFrameActionsNotificationTask(TWeakPtr<SNotificationItem> InNotificationItemPtr, SNotificationItem::ECompletionState InCompletionState, const FText& InText, const FText& InLinkText = FText(), bool InExpireAndFadeout = true, const TOptional<FText>& InSubText = TOptional<FText>())
		: CompletionState(InCompletionState)
		, NotificationItemPtr(InNotificationItemPtr)
		, Text(InText)
		, LinkText(InLinkText)
		, bExpireAndFadeout(InExpireAndFadeout)
		, SubText(InSubText)
	{ }

	static void HandleHyperlinkNavigate()
	{
		FMessageLog("PackagingResults").Open(EMessageSeverity::Error, true);
	}

	static void HandleDismissButtonClicked()
	{
		TSharedPtr<SNotificationItem> NotificationItem = ExpireNotificationItemPtr.Pin();
		if (NotificationItem.IsValid())
		{

			NotificationItem->SetExpireDuration(0.0f);
			NotificationItem->SetFadeOutDuration(0.0f);
			NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
			NotificationItem->ExpireAndFadeout();
			ExpireNotificationItemPtr.Reset();
		}
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		if ( NotificationItemPtr.IsValid() )
		{
			if (CompletionState == SNotificationItem::CS_Fail)
			{
				GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));
			}
			else
			{
				GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileSuccess_Cue.CompileSuccess_Cue"));
			}
			
			TSharedPtr<SNotificationItem> NotificationItem = NotificationItemPtr.Pin();
			NotificationItem->SetText(Text);

			if (SubText.IsSet())
			{
				NotificationItem->SetSubText(SubText.GetValue());
			}

			if (!LinkText.IsEmpty())
			{
				FText VLinkText(LinkText);
				const TAttribute<FText> Message = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([VLinkText]()
				{
					return VLinkText;
				}));

				NotificationItem->SetHyperlink(FSimpleDelegate::CreateStatic(&HandleHyperlinkNavigate), Message);

			}
			
			ExpireNotificationItemPtr = NotificationItem;
			if (bExpireAndFadeout)
			{
				NotificationItem->SetExpireDuration(6.0f);
				NotificationItem->SetFadeOutDuration(0.5f);
				NotificationItem->SetCompletionState(CompletionState);
				NotificationItem->ExpireAndFadeout();
			}
			else
			{
				// Handling the notification expiration in callback
				NotificationItem->SetCompletionState(CompletionState);
			}

			// Since the task was probably fairly long, we should try and grab the users attention if they have the option enabled.
			const UEditorPerProjectUserSettings* SettingsPtr = GetDefault<UEditorPerProjectUserSettings>();
			if (SettingsPtr->bGetAttentionOnUATCompletion)
			{
				IMainFrameModule* MainFrame = FModuleManager::LoadModulePtr<IMainFrameModule>("MainFrame");
				if (MainFrame != nullptr)
				{
					TSharedPtr<SWindow> ParentWindow = MainFrame->GetParentWindow();
					if (ParentWindow != nullptr)
					{
						ParentWindow->DrawAttention(FWindowDrawAttentionParameters(EWindowDrawAttentionRequestType::UntilActivated));
					}
				}
			}
		}
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::GameThread; }
	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMainFrameActionsNotificationTask, STATGROUP_TaskGraphTasks);
	}

private:

	static TWeakPtr<SNotificationItem> ExpireNotificationItemPtr;

	SNotificationItem::ECompletionState CompletionState;
	TWeakPtr<SNotificationItem> NotificationItemPtr;
	FText Text;
	FText LinkText;
	bool bExpireAndFadeout;
	TOptional<FText> SubText;
};

TWeakPtr<SNotificationItem> FMainFrameActionsNotificationTask::ExpireNotificationItemPtr;

/**
* Helper class to deal with packaging issues encountered in UAT.
**/
class FPackagingErrorHandler
{

public:


private:

	/**
	* Create a message to send to the Message Log.
	*
	* @Param MessageString - The error we wish to send to the Message Log.
	* @Param MessageType - The severity of the message, i.e. error, warning etc.
	**/
	static void AddMessageToMessageLog(FString MessageString, EMessageSeverity::Type MessageType)
	{
		if ((MessageType == EMessageSeverity::Error || MessageType == EMessageSeverity::Warning))
		{
			FAssetData AssetData;

			// Parse the warning/error into an array and check whether there is an asset on the path
			TArray<FString> MessageArray;
			MessageString.ParseIntoArray(MessageArray, TEXT(": "), true);

			FString AssetPath = MessageArray.Num() > 0 ? MessageArray[0] : TEXT("");
			if (AssetPath.Len())
			{			
				// Convert from the asset's full path provided by UE_ASSET_LOG back to an AssetData, if possible
				FString LongPackageName;
				FPaths::NormalizeFilename(AssetPath);
				if (!FPaths::IsRelative(AssetPath) && FPackageName::TryConvertFilenameToLongPackageName(AssetPath, LongPackageName))
				{
					// Generate qualified asset path and query the registry
					AssetPath = LongPackageName + TEXT(".") + FPackageName::GetShortName(LongPackageName);
					FSoftObjectPath SoftAssetPath(AssetPath);
					if (!SoftAssetPath.IsNull())
					{
						static const FName AssetRegistryModuleName(TEXT("AssetRegistry"));
						FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryModuleName);
						AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(SoftAssetPath, true);
					}
				}
			}
	

			if (AssetData.IsValid())
			{
				// we have asset errors in the cook
				if (MessageType == EMessageSeverity::Error)
				{
					bHasAssetErrors = true;
				}

				FMessageLog Message("PackagingResults");
				TSharedRef<FTokenizedMessage> PackagingMsg = MessageType == EMessageSeverity::Error ? Message.Error() : Message.Warning();

				PackagingMsg->AddToken(FTextToken::Create(FText::FromString(MessageArray.Num() > 1 ? MessageArray[1] : MessageString)));

				if (AssetData.IsValid())
				{
					PackagingMsg->AddToken(FUObjectToken::Create(AssetData.GetAsset()));
				}

			}

		}			

		// note: CookResults:Warning: actually outputs some unhandled errors.
		FText MsgText = FText::FromString(MessageString);

		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(MessageType);
		Message->AddToken(FTextToken::Create(MsgText));

		FMessageLog MessageLog("PackagingResults");
		MessageLog.AddMessage(Message);
		
	}

	/**
	* Send Error to the Message Log.
	*
	* @Param MessageString - The error we wish to send to the Message Log.
	* @Param MessageType - The severity of the message, i.e. error, warning etc.
	**/
	static void SyncMessageWithMessageLog(FString MessageString, EMessageSeverity::Type MessageType)
	{
		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.SendPackageErrorToMessageLog"),
		STAT_FSimpleDelegateGraphTask_SendPackageErrorToMessageLog,
			STATGROUP_TaskGraphTasks);

		// Remove any new line terminators
		MessageString.ReplaceInline(TEXT("\r"), TEXT(""));
		MessageString.ReplaceInline(TEXT("\n"), TEXT(""));

		/**
		* Dispatch the error from packaging to the message log.
		**/
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateStatic(&FPackagingErrorHandler::AddMessageToMessageLog, MessageString, MessageType),
			GET_STATID(STAT_FSimpleDelegateGraphTask_SendPackageErrorToMessageLog),
			nullptr, ENamedThreads::GameThread
			);
	}

	// Whether there are asset errors in the cook, which can be navigated to in the content browser.
	static bool bHasAssetErrors;

public:

	/**
	* Determine if the output is an communication message we wish to process.
	*
	* @Param UATOutput - The current line of output from the UAT package process.
	**/
	static bool ProcessAndHandleCookMessageOutput(FString UATOutput)
	{
		FString LhsUATOutputMsg, ParsedCookIssue;
		if (UATOutput.Split(TEXT("Shaders left to compile "), &LhsUATOutputMsg, &ParsedCookIssue))
		{
			if (GShaderCompilingManager)
			{
				if (ParsedCookIssue.IsNumeric())
				{
					int32 ShadersLeft = FCString::Atoi(*ParsedCookIssue);
					GShaderCompilingManager->SetExternalJobs(ShadersLeft);
				}
			}
			return false;
		}
		return true;
	}

	/**
	* Determine if the output is an error we wish to send to the Message Log.
	*
	* @Param UATOutput - The current line of output from the UAT package process.
	**/
	static void ProcessAndHandleCookErrorOutput(FString UATOutput)
	{
		FString LhsUATOutputMsg, ParsedCookIssue;

		// note: CookResults:Warning: actually outputs some unhandled errors.
		if (UATOutput.Split(TEXT("CookResults:Warning: "), &LhsUATOutputMsg, &ParsedCookIssue) )
		{
			SyncMessageWithMessageLog(ParsedCookIssue, EMessageSeverity::Warning);
		}
		else if (UATOutput.Split(TEXT("CookResults:Error: "), &LhsUATOutputMsg, &ParsedCookIssue) )
		{
			SyncMessageWithMessageLog(ParsedCookIssue, EMessageSeverity::Error);
		}
		else if (UATOutput.Split(TEXT("Warning: "), &LhsUATOutputMsg, &ParsedCookIssue))
		{
			SyncMessageWithMessageLog(ParsedCookIssue, EMessageSeverity::Warning);
		}
		else if (UATOutput.Split(TEXT("Error: "), &LhsUATOutputMsg, &ParsedCookIssue))
		{
			SyncMessageWithMessageLog(ParsedCookIssue, EMessageSeverity::Error);
		}
	}

	/**
	* Send the UAT Packaging error message to the Message Log.
	*
	* @Param ErrorCode - The UAT return code we received and wish to display the error message for.
	**/
	static void SendPackagingErrorToMessageLog(int32 ErrorCode)
	{
		SyncMessageWithMessageLog(FEditorAnalytics::TranslateErrorCode(ErrorCode), EMessageSeverity::Error);
	}

	/**
	* Get Whether the UAT process returned asset errors via the log
	**/
	static bool GetHasAssetErrors() { return bHasAssetErrors; }

	/**
	* Clear the asset error state for the UAT process
	**/
	static void ClearAssetErrors() { bHasAssetErrors = false; }

};

bool FPackagingErrorHandler::bHasAssetErrors = false;

DECLARE_CYCLE_STAT(TEXT("Requesting FUATHelperModule::HandleUatProcessCompleted message dialog to present the error message"), STAT_FUATHelperModule_HandleUatProcessCompleted_DialogMessage, STATGROUP_TaskGraphTasks);


class FUATHelperModule : public IUATHelperModule
{
public:
	FUATHelperModule()
	{
	}

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

	virtual void CreateUatTask( const FString& CommandLine, const FText& PlatformDisplayName, const FText& TaskName, const FText &TaskShortName, const FSlateBrush* TaskIcon, const TArray<FAnalyticsEventAttribute>* OptionalAnalyticsParamArray, UatTaskResultCallack ResultCallback, const FString& ResultLocation, const FText& ExtraSubText)
	{
		// If this is a packaging or cooking task, clear the PackagingResults log
		if (!TaskShortName.CompareToCaseIgnored(FText::FromString(TEXT("Packaging"))) || 
			!TaskShortName.CompareToCaseIgnored(FText::FromString(TEXT("Cooking"))))
		{
			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			if (MessageLogModule.IsRegisteredLogListing(TEXT("PackagingResults")))
			{
				TSharedRef<IMessageLogListing> PackagingResultsListing = MessageLogModule.GetLogListing(TEXT("PackagingResults"));
				PackagingResultsListing->ClearMessages();
			}
		}

		FGameProjectGenerationModule& GameProjectModule = FModuleManager::LoadModuleChecked<FGameProjectGenerationModule>(TEXT("GameProjectGeneration"));
		bool bHasCode = GameProjectModule.Get().ProjectHasCodeFiles();
		
		// make sure UAT exists
		if (!FPaths::FileExists(FSerializedUATProcess::GetUATPath()))
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("File"), FText::FromString(FSerializedUATProcess::GetUATPath()));
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("RequiredFileNotFoundMessage", "A required file could not be found:\n{File}"), Arguments));

			TArray<FAnalyticsEventAttribute> ParamArray;
			ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), 0.0));
			FString EventName = (CommandLine.Contains(TEXT("-package")) ? TEXT("Editor.Package") : TEXT("Editor.Cook"));
			FEditorAnalytics::ReportEvent(EventName + TEXT(".Failed"), PlatformDisplayName.ToString(), bHasCode, EAnalyticsErrorCodes::UATNotFound, ParamArray);
			
			return;
		}

		TSharedPtr<FSerializedUATProcess> UatProcess = MakeShareable(new FSerializedUATProcess(CommandLine));

		FPackagingErrorHandler::ClearAssetErrors();

		// create notification item
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Platform"), PlatformDisplayName);
		Arguments.Add(TEXT("TaskName"), TaskName);
		FText NotificationFormat = (PlatformDisplayName.IsEmpty()) ? LOCTEXT("UatTaskInProgressNotificationNoPlatform", "{TaskName}...") : LOCTEXT("UatTaskInProgressNotification", "{TaskName} for {Platform}...");
		FNotificationInfo Info( FText::Format( NotificationFormat, Arguments) );
		
		Info.Image = TaskIcon;
		Info.bFireAndForget = false;
		Info.FadeOutDuration = 0.0f;
		Info.ExpireDuration = 0.0f;
		Info.Hyperlink = FSimpleDelegate::CreateStatic(&FUATHelperModule::HandleUatHyperlinkNavigate);
		Info.HyperlinkText = LOCTEXT("ShowOutputLogHyperlink", "Show Output Log");
		Info.SubText = ExtraSubText;
		Info.ButtonDetails.Add(
			FNotificationButtonInfo(
				LOCTEXT("UatTaskCancel", "Cancel"),
				LOCTEXT("UatTaskCancelToolTip", "Cancels execution of this task."),
				FSimpleDelegate::CreateStatic(&FUATHelperModule::HandleUatCancelButtonClicked, UatProcess),
				SNotificationItem::CS_Pending
			)
		);
		Info.ButtonDetails.Add(
			FNotificationButtonInfo(
				LOCTEXT("UatTaskDismiss", "Dismiss"),
				FText(),
				FSimpleDelegate::CreateStatic(&FMainFrameActionsNotificationTask::HandleDismissButtonClicked),
				SNotificationItem::CS_Fail
			)
		);

		TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
		TSharedPtr<SNotificationItem> OldNotification = NotificationItemPtr.Pin();
		if(OldNotification.IsValid())
		{
			OldNotification->Fadeout();
		}

		if (!NotificationItem.IsValid())
		{
			return;
		}

		FString EventName = (CommandLine.Contains(TEXT("-package")) ? TEXT("Editor.Package") : TEXT("Editor.Cook"));
		if (OptionalAnalyticsParamArray != nullptr)
		{
			FEditorAnalytics::ReportEvent(EventName + TEXT(".Start"), PlatformDisplayName.ToString(), bHasCode, *OptionalAnalyticsParamArray);
		}
		else
		{
			FEditorAnalytics::ReportEvent(EventName + TEXT(".Start"), PlatformDisplayName.ToString(), bHasCode);
		}

		NotificationItem->SetCompletionState(SNotificationItem::CS_Pending);

		// launch the packager
		NotificationItemPtr = NotificationItem;

		EventData Data;
		Data.StartTime = FPlatformTime::Seconds();
		Data.EventName = EventName;
		Data.bProjectHasCode = bHasCode;
		Data.ResultCallback = ResultCallback;
		UatProcess->OnCanceled().BindStatic(&FUATHelperModule::HandleUatProcessCanceled, NotificationItemPtr, PlatformDisplayName, TaskShortName, Data);
		UatProcess->OnCompleted().BindStatic(&FUATHelperModule::HandleUatProcessCompleted, NotificationItemPtr, PlatformDisplayName, TaskShortName, Data, ResultLocation);
		UatProcess->OnOutput().BindStatic(&FUATHelperModule::HandleUatProcessOutput, NotificationItemPtr, PlatformDisplayName, TaskShortName);
		UatProcess->OnLaunchFailed().BindStatic(&FUATHelperModule::HandleUatLaunchFailed, NotificationItemPtr, PlatformDisplayName, TaskShortName, Data);

		TWeakPtr<FSerializedUATProcess> UatProcessPtr(UatProcess);
		FEditorDelegates::OnShutdownPostPackagesSaved.Add(FSimpleDelegate::CreateStatic(&FUATHelperModule::HandleUatCancelButtonClicked, UatProcessPtr));

		UatProcess->Launch();
		GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileStart_Cue.CompileStart_Cue"));
	}

	static void HandleUatHyperlinkNavigate()
	{
		FOutputLogModule& OutputLogModule = FOutputLogModule::Get();
		OutputLogModule.FocusOutputLog();
	}

	static void HandleUatResultHyperlinkNavigate(FString ResultLocation)
	{
		if (!ResultLocation.IsEmpty())
		{
			FPlatformProcess::ExploreFolder(*(ResultLocation));
		}
	}

	static void HandleUatCancelButtonClicked(TSharedPtr<FSerializedUATProcess> PackagerProcess)
	{
		if ( PackagerProcess.IsValid() )
		{
			PackagerProcess->Cancel(true);
		}
	}

	static void HandleUatCancelButtonClicked(TWeakPtr<FSerializedUATProcess> PackagerProcessPtr)
	{
		TSharedPtr<FMonitoredProcess> PackagerProcess = PackagerProcessPtr.Pin();
		if ( PackagerProcess.IsValid() )
		{
			PackagerProcess->Cancel(true);
		}
	}

	static void HandleUatProcessCanceled(TWeakPtr<SNotificationItem> NotificationItemPtr, FText PlatformDisplayName, FText TaskName, EventData Event)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Platform"), PlatformDisplayName);
		Arguments.Add(TEXT("TaskName"), TaskName);

		const FText NoLinkText;
		const FText ClearedSubText;
		const bool bExpireAndFadeout = true;
		TGraphTask<FMainFrameActionsNotificationTask>::CreateTask().ConstructAndDispatchWhenReady(
			NotificationItemPtr,
			SNotificationItem::CS_Fail,
			FText::Format(LOCTEXT("UatProcessFailedNotification", "{TaskName} canceled!"), Arguments),
			NoLinkText,
			bExpireAndFadeout,
			ClearedSubText
			);

		TArray<FAnalyticsEventAttribute> ParamArray;
		const double TimeSec = FPlatformTime::Seconds() - Event.StartTime;
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), TimeSec));
		FEditorAnalytics::ReportEvent(Event.EventName + TEXT(".Canceled"), PlatformDisplayName.ToString(), Event.bProjectHasCode, ParamArray);
		if (Event.ResultCallback)
		{
			Event.ResultCallback(TEXT("Canceled"), TimeSec);
		}
		if (GShaderCompilingManager)
		{
			GShaderCompilingManager->SetExternalJobs(0);
		}
		//	FMessageLog("PackagingResults").Warning(FText::Format(LOCTEXT("UatProcessCanceledMessageLog", "{TaskName} for {Platform} canceled by user"), Arguments));
	}

	static void HandleUatProcessCompleted(int32 ReturnCode, TWeakPtr<SNotificationItem> NotificationItemPtr, FText PlatformDisplayName, FText TaskName, EventData Event, FString ResultLocation)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Platform"), PlatformDisplayName);
		Arguments.Add(TEXT("TaskName"), TaskName);
		const double TimeSec = FPlatformTime::Seconds() - Event.StartTime;

		if ( ReturnCode == 0 )
		{
			if (!ResultLocation.IsEmpty())
			{
				if (TSharedPtr<SNotificationItem> SharedNotificationItemPtr = NotificationItemPtr.Pin())
				{
					SharedNotificationItemPtr->SetHyperlink(FSimpleDelegate::CreateStatic(&FUATHelperModule::HandleUatResultHyperlinkNavigate, ResultLocation), LOCTEXT("ShowOutputLocation", "Show in Explorer"));
				}

			}
			
			TGraphTask<FMainFrameActionsNotificationTask>::CreateTask().ConstructAndDispatchWhenReady(
				NotificationItemPtr,
				SNotificationItem::CS_Success,
				FText::Format(LOCTEXT("UatProcessSucceededNotification", "{TaskName} complete!"), Arguments)
				);

			TArray<FAnalyticsEventAttribute> ParamArray;
			ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), TimeSec));
			FEditorAnalytics::ReportEvent(Event.EventName + TEXT(".Completed"), PlatformDisplayName.ToString(), Event.bProjectHasCode, ParamArray);
			if (Event.ResultCallback)
			{
				Event.ResultCallback(TEXT("Completed"), TimeSec);
			}

			//		FMessageLog("PackagingResults").Info(FText::Format(LOCTEXT("UatProcessSuccessMessageLog", "{TaskName} for {Platform} completed successfully"), Arguments));
		}
		else
		{
			const FText ResultsLinkText = FPackagingErrorHandler::GetHasAssetErrors() ? LOCTEXT("ShowResultsLogHyperlink", "Show Results Log") : FText();
			const FText ClearedSubText;
			const bool bExpireAndFadeout = false;
			TGraphTask<FMainFrameActionsNotificationTask>::CreateTask().ConstructAndDispatchWhenReady(
				NotificationItemPtr,
				SNotificationItem::CS_Fail,
				FText::Format(LOCTEXT("PackagerFailedNotification", "{TaskName} failed!"), Arguments),
				ResultsLinkText,
				bExpireAndFadeout,
				ClearedSubText
				);

			TArray<FAnalyticsEventAttribute> ParamArray;
			ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), TimeSec));
			FEditorAnalytics::ReportEvent(Event.EventName + TEXT(".Failed"), PlatformDisplayName.ToString(), Event.bProjectHasCode, ReturnCode, ParamArray);
			if (Event.ResultCallback)
			{
				Event.ResultCallback(TEXT("Failed"), TimeSec);
			}

			// Send the error to the Message Log.
			if ( TaskName.EqualTo(LOCTEXT("PackagingTaskName", "Packaging")) )
			{
				FPackagingErrorHandler::SendPackagingErrorToMessageLog(ReturnCode);
			}

			// Present a message dialog if we want the error message to be prominent.
			if ( FEditorAnalytics::ShouldElevateMessageThroughDialog(ReturnCode) )
			{
				FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
					FSimpleDelegateGraphTask::FDelegate::CreateLambda([=] (){
					FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(FEditorAnalytics::TranslateErrorCode(ReturnCode)));
				}),
					GET_STATID(STAT_FUATHelperModule_HandleUatProcessCompleted_DialogMessage),
					nullptr,
					ENamedThreads::GameThread
					);
			}			

			//		FMessageLog("PackagingResults").Info(FText::Format(LOCTEXT("UatProcessFailedMessageLog", "{TaskName} for {Platform} failed"), Arguments));
		}
		if (GShaderCompilingManager)
		{
			GShaderCompilingManager->SetExternalJobs(0);
		}
	}

	static ELogVerbosity::Type GetVerbosityFromStructuredLogString(FStringView Level)
	{
		if (Level == TEXT("Information"))
		{
			return ELogVerbosity::Log; // could be Display
		}
		else if (Level == TEXT("Debug"))
		{
			return ELogVerbosity::Verbose; // could be VeryVerbose
		}
		else if (Level == TEXT("Warning"))
		{
			return ELogVerbosity::Warning;
		}
		else if (Level == TEXT("Error"))
		{
			return ELogVerbosity::Error;
		}
		else if (Level == TEXT("Critical"))
		{
			return ELogVerbosity::Error; // don't report external criticals as Fatal because they will then crash our process
		}
		else
		{
			return ELogVerbosity::Log;
		}
	}

	static void HandleUatProcessOutput(FString Output, TWeakPtr<SNotificationItem> NotificationItemPtr, FText PlatformDisplayName, FText TaskName)
	{
		if (Output.IsEmpty() || Output.Equals("\r"))
		{
			return;
		}

		ELogVerbosity::Type Verbosity = ELogVerbosity::Log;
		FString Message = Output;
		bool bStructuredMessage = false;

		// attempt to parse structured logging
		TSharedRef< TJsonReader<> > JsonReader = FJsonStringReader::Create(Output);
		TSharedPtr<FJsonObject> JsonData;
		if (FJsonSerializer::Deserialize(JsonReader, JsonData))
		{
			TSharedPtr<FJsonValue> LevelField = JsonData->TryGetField(TEXT("level"));
			TSharedPtr<FJsonValue> MessageField = JsonData->TryGetField(TEXT("message"));

			FString LevelString;
			FString MessageString;

			if (LevelField && LevelField->TryGetString(LevelString)
				&& MessageField && MessageField->TryGetString(Message))
			{
				Verbosity = GetVerbosityFromStructuredLogString(LevelString);
				bStructuredMessage = true;
			}

			// TODO add support for "format" field
		}

		bool bDisplayLog = true;
		if (TaskName.EqualTo(LOCTEXT("PackagingTaskName", "Packaging")))
		{
			// Deal with any cook messages that may have been encountered.
			bDisplayLog = FPackagingErrorHandler::ProcessAndHandleCookMessageOutput(Output);
		}
		if (bDisplayLog)
		{
			FMsg::Logf(__FILE__, __LINE__,
				UATHelper.GetCategoryName(),
				Verbosity,
				TEXT("%s (%s): %s"), *TaskName.ToString(), *PlatformDisplayName.ToString(), *Message);
		}

		// deal with non-structured packaging/cooking warnings and errors
		if (!bStructuredMessage && (TaskName.EqualTo(LOCTEXT("PackagingTaskName", "Packaging")) || TaskName.EqualTo(LOCTEXT("CookingTaskName", "Cooking"))))
		{
			FPackagingErrorHandler::ProcessAndHandleCookErrorOutput(Output);
		}
	}

	static void HandleUatLaunchFailed(TWeakPtr<SNotificationItem> NotificationItemPtr, FText PlatformDisplayName, FText TaskName, EventData Event)
	{
		GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));

		TGraphTask<FMainFrameActionsNotificationTask>::CreateTask().ConstructAndDispatchWhenReady(
			NotificationItemPtr,
			SNotificationItem::CS_Fail,
			LOCTEXT("UatLaunchFailedNotification", "Failed to launch Unreal Automation Tool (UAT)!")
		);

		TArray<FAnalyticsEventAttribute> ParamArray;
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), 0.0));
		FEditorAnalytics::ReportEvent(Event.EventName + TEXT(".Failed"), PlatformDisplayName.ToString(), Event.bProjectHasCode, EAnalyticsErrorCodes::UATLaunchFailure, ParamArray);
		if (Event.ResultCallback)
		{
			Event.ResultCallback(TEXT("FailedToStart"), 0.0f);
		}
	}

private:
	TWeakPtr<SNotificationItem> NotificationItemPtr;

};

IMPLEMENT_MODULE(FUATHelperModule, UATHelper)


#undef LOCTEXT_NAMESPACE
